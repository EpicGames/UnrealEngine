<?xml version="1.0"?>
<configuration>
  <startup>
    <supportedRuntime version="v4.0" sku=".NETFramework,Version=v4.6.2"/>
  </startup>
  <runtime>
    <Thread_UseAllCpuGroups enabled="true"/>
    <GCCpuGroup enabled="true"/>
    <gcServer enabled="true"/>
  </runtime>
</configuration>
