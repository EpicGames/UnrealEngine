// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace UnrealAndroidFileTool
{
	class Program
	{
		private static string AppVersion = "1.0.9";
		private static ILogger Logger = Log.Logger;

		static void ShowHelp(string command)
		{
			if (command == "")
			{
				Logger.LogInformation("UnrealAndroidFileTool version {0}\n" +
					"Copyright Epic Games, Inc. All Rights Reserved.\n\n" +
					"UnrealAndroidFileTool [-s Device] [-ip ipAddress] [-t port] [-p PackageName] [-k token] command\n\n" +
					"Commands:\n\n" +
					"devices                  Show connected devices (@ prefix for authorized).\n" +
					"packages                 Show list of packages with receiver enabled.\n" +
					"stop-all [-w]            Send stop request to all packages with receivers.\n" +
					"help [command]           Get help about a command.\n" +
					"shell                    Start interactive mode.\n" +
					"quit                     Exit interactive mode.\n" +
					"exit                     Exit interactive mode.\n" +
					"terminate                Stop remote server.\n" +
					"query [key]              Show all variables or a specific key.\n" +
					"getprop [key]            Get property for key.\n" + 
					"cd path                  Set base directory (value of ^^).\n" +
					"pwd                      Show current base directory.\n" +
					"direxists path           Returns true if directory path exists.\n" +
//					"dirlist [RSA:]path       Show contents of directory path.\n" +
					"ls [-lsRf] path          Show contents of directory path.\n" +
//					"dirlistflat path         Show recursive contents of directories grouped.\n" +
//					"dircreate path           Create directory path (recursive).\n" +
					"mkdir path               Create directory path (recursive).\n" +
//					"dirdelete path           Delete directory path (recursive).\n" +
					"rmdir path               Delete directory path (recursive).\n" +
					"fileexists file          Returns true if file exists.\n" +
//					"filedelete file          Delete file.\n" +
					"rm file                  Delete file.\n" +
//					"filecopy src dst         Copies file from src to dst on device.\n" +
					"cp src dst               Copies file from src to dst on device.\n" +
//					"filemove src dst         Moves file from src to dst on device.\n" +
					"mv src dst               Moves file from src to dst on device.\n" +
//					"fileread src dst         Pulls src file from device to local dst.\n" +
					"pull src dst             Pulls src file from device to local dst.\n" +
					"pulldir src dst          Pulls src directory from device to local dst.\n" +
//					"filewrite src dst        Pushes local src file to device dst.\n" +
					"push [-c] src dst        Pushes local src file or directories to device dst.\n" +
					"command [-f file] [data] Writes [file] or [data] to commandline file, or shows contents.\n" +
					"addcommand [data]        Adds data to commandline file.\n" +
					"delcommand [data]        Removes data from commandline file.\n" +
					"cat file                 Writes file contents from device to output.\n" +
					"deploy [-c] file         Reads a text file with deployment source/dest pairs.\n" +
					"\n" +
					"Note: keys may be used at start of a path as shortcut.\n" +
					"   Ex: fileread ^logfile game.log   - pulls logfile.\n" +
					"       rm ^project/Manifest.txt     - deletes Manifest.txt from project dir.\n\n"
					, AppVersion);
				return;
			}
			if (command == "command")
			{
				Logger.LogInformation("No Args\t\tShow contents of commandline file");
				Logger.LogInformation("-f [file]\tWrites contents of [file] to commandline file");
				Logger.LogInformation("[data]\t\tOverwrites commandline file with [data]");
				return;
			}
			if (command == "ls")
			{
				Logger.LogInformation("-l\tlist permissions");
				Logger.LogInformation("-s\tlist size");
				Logger.LogInformation("-R\tlist recursive directory tree");
				Logger.LogInformation("-f\tlist flat\n");
				return;
			}
			if (command == "stop-all")
			{
				Logger.LogInformation("-w\twait for all listen binds to terminate");
				return;
			}
			if (command == "push")
			{
				Logger.LogInformation("-c\tcompress files\n");
				return;
			}
			if (command == "deploy")
			{
				Logger.LogInformation("-c\tcompress files\n");
				return;
			}
		}

		static string FixPath(AndroidFileClient client, string path)
		{
			if (path.StartsWith("^^"))
			{
				string cmd = client.Query("^^");
				path = client.Query("^^") + path.Substring(2);
			}
			else if (!(path.StartsWith("^") || path.StartsWith("/")))
			{
				string cwd = client.Query("^^");
				if (cwd != "")
				{
					path = cwd + "/" + path;
				}
			}

			// remove .. sets by trimming out prior directory
			int dotdotIndex = path.IndexOf("..");
			while (dotdotIndex > 0)
			{
				int slashIndex = path.Substring(0, dotdotIndex - 1).LastIndexOf("/");
				if (slashIndex < 0)
				{
					break;
				}
				path = path.Substring(0, slashIndex) + path.Substring(dotdotIndex + 2);
				dotdotIndex = path.IndexOf("..");
			}

			return path;
		}

		static int ProcessCommand(AndroidFileClient client, bool bInShell, string[] args, int ArgIndex)
		{
			if (ArgIndex < args.Length && (args[ArgIndex] == "help" || args[ArgIndex] == "?"))
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					ShowHelp(args[ArgIndex++]);
				}
				else
				{
					ShowHelp("");
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "shell")
			{
				ArgIndex++;
				while (true)
				{
					Console.Write("shell> ");
					string? input = Console.ReadLine();
					if (input == null || input == "exit" || input == "quit")
					{
						return ArgIndex;
					}

					if (input != "")
					{
						string[] split = input.Split(new char[] { ' ', '\t' });
						if (split.Length > 0)
						{
							if (ProcessCommand(client, true, split, 0) == -1)
							{
								return -1;
							}
						}
					}
				}
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "packages")
			{
				ArgIndex++;
				List<string> Receivers = AndroidFileClient.GetInstalledReceivers(client.GetDevice());
				AndroidFileClient.GetInstalledActivities(client.GetDevice()).ForEach(item => { if (!Receivers.Contains(item)) Receivers.Add(item); });
				if (Receivers.Count > 0)
				{
					foreach (string Line in Receivers)
					{
						Logger.LogInformation("{0}", Line);
					}
					return ArgIndex;
				}
				Logger.LogInformation("No packages found with receiver.");
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "stop-all")
			{
				ArgIndex++;
				bool bWait = false;

				if (ArgIndex < args.Length && args[ArgIndex].StartsWith("-w"))
				{
					bWait = true;
				}

				client.CloseConnection();
				AndroidFileClient.StopAnyServers(client.GetDevice(), client.GetServerPort(), bWait);
				System.Environment.Exit(0);
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "stats")
			{
				ArgIndex++;
				client.Stats_Report();
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "terminate")
			{
				ArgIndex++;
				client.TerminateServer();
				Logger.LogInformation("Terminated server");
				return -1;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "pwd")
			{
				ArgIndex++;
				string result = client.Query("^^");
				Logger.LogInformation("{0}", result);
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "query")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string result = client.Query(args[ArgIndex++]);
					Logger.LogInformation("{0}", result);
				}
				else
				{
					string result = client.Query("");
					string[] lines = result.Split('\n');
					foreach (string line in lines)
					{
						string[] parts = line.Split("\t");
						if (parts.Length > 1)
						{
							Logger.LogInformation("{0,-20}{1}", parts[0], parts[1]);
						}
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "getprop")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string result = client.GetProp(args[ArgIndex++]);
					Logger.LogInformation("{0}", result);
				}
				else
				{
					string result = client.GetProp("");
					Logger.LogInformation("{0}", result);
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "cd")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.SetBaseDir(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", client.Query("^^"));
					return ArgIndex++;
				}
				else
				{
					string result = client.Query("^^");
					Logger.LogInformation("{0}", result);
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "direxists")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.DirExists(args[ArgIndex]);
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "ls")
			{
				ArgIndex++;
				string path = "^^";

				// process flags
				bool bOptions = false;
				bool bAttributes = false;
				bool bRecursive = false;
				bool bSize = false;
				bool bFlat = false;
				while (ArgIndex < args.Length && args[ArgIndex].StartsWith("-"))
				{
					if (args[ArgIndex].Contains("l"))
					{
						bAttributes = true;
						bOptions = true;
					}
					if (args[ArgIndex].Contains("R"))
					{
						bRecursive = true;
						bOptions = true;
					}
					if (args[ArgIndex].Contains("s"))
					{
						bSize = true;
						bOptions = true;
					}
					if (args[ArgIndex].Contains("f"))
					{
						bFlat = true;
					}
					ArgIndex++;
				}
				if (ArgIndex < args.Length)
				{
					path = args[ArgIndex];
					ArgIndex++;
				}
				path = FixPath(client, path);
				if (path == "^^")
				{
					path = client.Query("^^");
				}
				string result;
				if (bRecursive && bFlat)
				{
					result = client.DirListFlat(path);
				}
				else
				{
					result = client.DirList((bRecursive ? "R" : "") + (bSize ? "S" : "") + (bAttributes ? "A" : "") + (bOptions ? ":" : "") + path);
				}
				Logger.LogInformation("{0}", result);
				return ArgIndex++;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "dirlist")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string result = client.DirList(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", result);
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "dirlistflat"))
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string result = client.DirListFlat(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", result);
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "dircreate" || args[ArgIndex] == "mkdir"))
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.DirCreate(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "dirdelete" || args[ArgIndex] == "rmdir"))
			{
				ArgIndex++;
				if (ArgIndex < args.Length && args[ArgIndex] == "-r")
				{
					ArgIndex++;
					if (ArgIndex < args.Length)
					{
						bool result = client.DirDeleteRecurse(FixPath(client, args[ArgIndex]));
						Logger.LogInformation("{0}", (result ? "true" : "false"));
						if (!bInShell && !result)
						{
							client.CloseConnection();
							System.Environment.Exit(1);

						}
					}
					else
					{
						Logger.LogError("missing path");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
				}
				else if (ArgIndex < args.Length)
				{
					bool result = client.DirDelete(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "dirdeleterecurse")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.DirDeleteRecurse(args[ArgIndex]);
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "fileexists")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.FileExists(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "filedelete" || args[ArgIndex] == "rm"))
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool result = client.FileDelete(FixPath(client, args[ArgIndex]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "filecopy" || args[ArgIndex] == "cp"))
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					bool result = client.FileCopy(FixPath(client, args[ArgIndex]), FixPath(client, args[ArgIndex+1]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex +=2;
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "filemove" || args[ArgIndex] == "mv"))
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					bool result = client.FileMove(FixPath(client, args[ArgIndex]), FixPath(client, args[ArgIndex + 1]));
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
					return ArgIndex += 2;
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && (args[ArgIndex] == "fileread" || args[ArgIndex] == "pull"))
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					bool result = client.FileRead(args[ArgIndex], args[ArgIndex + 1]);
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex += 2;
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "pulldir")
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					string result = client.DirListFlat(FixPath(client, args[ArgIndex]));
					if (result == "(null)")
					{
						Logger.LogError("false");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
						return ArgIndex += 2;
					}
					// first line is the src path base
					string[] Lines = result.Split('\n');
					string BaseSrcPath = Lines[0].Substring(0, Lines[0].Length - 1);
					int BaseSrcPathLength = BaseSrcPath.Length;
					string DestBasePath = "";

					foreach (string Line in Lines)
					{
						if (Line == "")
						{
							continue;
						}
						if (Line.EndsWith(":"))
						{
							DestBasePath = Line.Substring(BaseSrcPathLength, Line.Length - BaseSrcPathLength - 1);
							if (DestBasePath.StartsWith("/"))
							{
								DestBasePath = DestBasePath.Substring(1);
							}
							// make directories if don't exist
							string TargetDir = args[ArgIndex + 1] + (DestBasePath.Length > 0 ? "/" + DestBasePath : "");
							try
							{
								System.IO.Directory.CreateDirectory(TargetDir);
							}
							catch (Exception)
							{
								Logger.LogError("Failed to create {0}\nfalse", TargetDir);
								if (!bInShell)
								{
									client.CloseConnection();
									System.Environment.Exit(1);
								}
								return ArgIndex += 2;
							}
							continue;
						}
						string Src = BaseSrcPath + "/" + (DestBasePath.Length > 0 ? DestBasePath + "/" : "") + Line;
						string Dest = args[ArgIndex + 1] + "/" + (DestBasePath.Length > 0 ? DestBasePath + "/" : "") + Line;
						bool copyresult = client.FileRead(Src, Dest);
						if (!bInShell && !copyresult)
						{
							Logger.LogError("false");
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
					Logger.LogInformation("true");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex += 2;
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);

					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "filewrite")
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					//Logger.LogInformation("Src: '{0}', Dest: '{1}'", args[ArgIndex], args[ArgIndex + 1]);
					bool result = client.FileWrite(args[ArgIndex], args[ArgIndex + 1]);
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex += 2;
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "push")
			{
				ArgIndex++;
				if (ArgIndex + 1 < args.Length)
				{
					bool result = false;
					bool bCompress = false;

					if (args[ArgIndex] == "-c")
					{
						bCompress = true;
						ArgIndex++;
					}
					if (ArgIndex + 1 < args.Length)
					{
						string source = args[ArgIndex];
						string dest = args[ArgIndex + 1];
						ArgIndex += 2;
						try
						{
							FileAttributes attributes = File.GetAttributes(source);
							if ((attributes & FileAttributes.Directory) == FileAttributes.Directory)
							{
								result = client.PushDirectories(source, dest, bCompress, 1);
							}
							else
							{
								result = client.PushFile(source, dest, bCompress, 1);
							}
						}
						catch (Exception)
						{
							Logger.LogInformation("false");
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);

							}
							return ArgIndex;
						}

						Logger.LogInformation("{0}", (result ? "true" : "false"));
						if (!bInShell && !result)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
						return ArgIndex;
					}
					else
					{
						Logger.LogError("missing paths");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
				}
				else
				{
					Logger.LogError("missing paths");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "command")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string command = args[ArgIndex++];
					if (command == "-f")
					{
						if (ArgIndex < args.Length && File.Exists(args[ArgIndex]))
						{
							string[] CommandLines = File.ReadAllLines(args[ArgIndex++]);
							command = "";
							if (CommandLines.Length > 0)
							{
								command = CommandLines[0];
							}
						}
						else
						{
							Logger.LogError("missing filename");
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);
							}
						}
					}
					else
					{
						while (ArgIndex < args.Length)
						{
							command = command + " " + args[ArgIndex++];
						}
					}
					bool result = client.FileWriteString(command, "^commandfile");
					Logger.LogInformation("{0}", (result ? "true" : "false"));
					if (!bInShell && !result)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					return ArgIndex;
				}
				else
				{
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead("^commandfile", TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							Console.WriteLine(myReader.ReadToEnd());
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", TempFilename, e.ToString());
						}
					}
					else
					{
						Logger.LogInformation("No commandline file found.");
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "addcommand")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string command = "";
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead("^commandfile", TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							string? line = myReader.ReadLine();
							command = (line != null) ? line : "";
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception)
						{
						}
					}
					while (ArgIndex < args.Length)
					{
						command = command + " " + args[ArgIndex++];
					}
					result = client.FileWriteString(command, "^commandfile");
					if (!bInShell && !result)
					{
						Logger.LogError("false");
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					Logger.LogInformation("New commandline: {0}", command);
					return ArgIndex;
				}
				else
				{
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead("^commandfile", TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							Console.WriteLine(myReader.ReadToEnd());
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", TempFilename, e.ToString());
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);
							}
						}
					}
					else
					{
						Logger.LogInformation("No commandline file found.");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);

						}
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "delcommand")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string command = "";
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead("^commandfile", TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							string? line = myReader.ReadLine();
							command = (line != null) ? line : "";
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", TempFilename, e.ToString());
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);
							}
						}
					}
					while (ArgIndex < args.Length)
					{
						command = command.Replace(args[ArgIndex++], "");
					}
					result = client.FileWriteString(command, "^commandfile");
					if (!bInShell && !result)
					{
						Logger.LogError("false");
						client.CloseConnection();
						System.Environment.Exit(1);
					}
					Logger.LogInformation("New commandline: {0}", command);
					return ArgIndex;
				}
				else
				{
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead("^commandfile", TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							Console.WriteLine(myReader.ReadToEnd());
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", TempFilename, e.ToString());
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);
							}
						}
					}
					else
					{
						Logger.LogInformation("No commandline file found.");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "cat")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					string TempFilename = Path.GetTempFileName();
					bool result = client.FileRead(FixPath(client, args[ArgIndex]), TempFilename);
					if (result)
					{
						try
						{
							StreamReader myReader = File.OpenText(TempFilename);
							Console.WriteLine(myReader.ReadToEnd());
							myReader.Close();
							File.Delete(TempFilename);
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", TempFilename, e.ToString());
						}
					}
					else
					{
						Logger.LogInformation("{0}", (result ? "true" : "false"));
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
					return ArgIndex++;
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "deploy")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					bool bCompress = false;

					if (args[ArgIndex] == "-c")
					{
						bCompress = true;
						ArgIndex++;
					}
					if (ArgIndex < args.Length)
					{
						try
						{
							StreamReader myReader = File.OpenText(args[ArgIndex]);
							ArgIndex++;

							string? SourceDir = myReader.ReadLine();
							string? DestDir = myReader.ReadLine();
							HashSet<string> EntriesToDeploy = new HashSet<string>();
							string? Entry;
							while ((Entry = myReader.ReadLine()) != null)
							{
								if (Entry != null)
								{
									EntriesToDeploy.Add(Entry);
								}
							}
							myReader.Close();
							if (SourceDir != null && DestDir != null)
							{
								client.Deploy(EntriesToDeploy, SourceDir, DestDir, bCompress, false, true, null);
							}
						}
						catch (Exception e)
						{
							Logger.LogError("Error reading temp file {0}, Exception: {1}", args[ArgIndex], e.ToString());
							if (!bInShell)
							{
								client.CloseConnection();
								System.Environment.Exit(1);
							}
						}
					}
					else
					{
						Logger.LogError("missing path");
						if (!bInShell)
						{
							client.CloseConnection();
							System.Environment.Exit(1);
						}
					}
				}
				else
				{
					Logger.LogError("missing path");
					if (!bInShell)
					{
						client.CloseConnection();
						System.Environment.Exit(1);
					}
				}
				return ArgIndex;
			}

			Logger.LogWarning("Unknown command");
			return ArgIndex;
		}

		static AndroidFileClient? CreateAndConnectClient(string Device, string PackageName, string Token, string IPAddress, int Port)
		{
			AndroidFileClient? client = new AndroidFileClient(Device, Port);
			if (client != null)
			{
				bool bSetActive = false;

				if (!client.OpenConnection(IPAddress))
				{
					if (PackageName == "")
					{
						Logger.LogError("Need package name to start server on {0}!", Device);
						return null;
					}
					else
					{
						Logger.LogInformation("Trying to start file server {0}", PackageName);
						if (!client.StartServer(PackageName, Token, IPAddress))
						{
							Logger.LogError("Unable to connect to " + Device);
							return null;
						}
						bSetActive = true;
					}
				}

				// verify connection to the correct server
				string DevicePackageName = client.Query("^packagename");
				if (DevicePackageName != PackageName)
				{
					if (PackageName == "")
					{
						Logger.LogError("Need package name to start server on {0}!", Device);
						client.CloseConnection();
						return null;
					}

					Logger.LogInformation("Connected to wrong server {0}, trying again", DevicePackageName);
					client.TerminateServer();

					Logger.LogInformation("Trying to start file server {0}", PackageName);
					if (!client.StartServer(PackageName, Token, IPAddress))
					{
						Logger.LogError("Failed to start server");
						return null;
					}
					bSetActive = true;
				}

				// remember active state for retries
				if (!bSetActive)
				{
					client.SetActive(PackageName, Token, IPAddress);
				}
				return client;
			}
			return null;
		}

		static void Main(string[] args)
		{
			string IPAddress = "";
			string Device = "";
			int Port = AndroidFileClient.GetDefaultPort();
			string PackageName = "";
			string Token = "";

			if (args.Length == 0 || (args.Length == 1 && args[0] == "help"))
			{
				ShowHelp("");
				System.Environment.Exit(0);
			}
			if (args.Length > 1 && args[0] == "help")
			{
				ShowHelp(args[1]);
				System.Environment.Exit(0);
			}

			int ArgIndex = 0;
			if (ArgIndex < args.Length && args[ArgIndex] == "-s")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					Device = "@" + args[ArgIndex];
					ArgIndex++;
				}
				else
				{
					Logger.LogError("Missing device id");
					System.Environment.Exit(1);
				}
			}

			if (ArgIndex < args.Length && args[ArgIndex] == "-p")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					PackageName = args[ArgIndex];
					ArgIndex++;
				}
				else
				{
					Logger.LogError("Missing package name");
					System.Environment.Exit(1);
				}
			}

			if (ArgIndex < args.Length && args[ArgIndex] == "-k")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					Token = args[ArgIndex];
					ArgIndex++;
				}
				else
				{
					Logger.LogError("Missing security token key");
					System.Environment.Exit(1);
				}
			}

			if (ArgIndex < args.Length && args[ArgIndex] == "-t")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					try
					{
						Port = int.Parse(args[ArgIndex]);
						ArgIndex++;
					}
					catch (Exception)
					{
						Logger.LogError("Invalid port");
						System.Environment.Exit(1);
					}
				}
				else
				{
					Logger.LogError("Missing port");
					System.Environment.Exit(1);
				}
			}

			if (ArgIndex < args.Length && args[ArgIndex] == "-ip")
			{
				ArgIndex++;
				if (ArgIndex < args.Length)
				{
					IPAddress = args[ArgIndex];
					ArgIndex++;
				}
				else
				{
					Logger.LogError("Missing IP address");
					System.Environment.Exit(1);
				}
			}

			if (ArgIndex < args.Length && args[ArgIndex] == "devices")
			{
				List<string> devices = new List<string>();
				AndroidFileClient.GetConnectedDevices(out devices);
				foreach (string DeviceName in devices)
				{
					Logger.LogInformation("{0}", DeviceName);
				}
				return;
			}

			if (Device == "")
			{
				// look for connected devices
				List<string> devices = new List<string>();
				AndroidFileClient.GetConnectedDevices(out devices);
				if (devices.Count == 0)
				{
					Logger.LogError("No connected devices");
				}
				else if (devices.Count == 1)
				{
					Device = devices[0];
				}
				else
				{
					Logger.LogError("Multiple devices connected, select one -s!");
					foreach (string DeviceName in devices)
					{
						Logger.LogInformation("{0}", DeviceName);
					}
				}
			}

			if (Device == "")
			{
				Logger.LogError("No devices attached!");
				System.Environment.Exit(0);
			}

			// check these here since may not have a running server to connect
			if (ArgIndex < args.Length && args[ArgIndex] == "packages")
			{
				ArgIndex++;
				List<string> Receivers = AndroidFileClient.GetInstalledReceivers(Device.Substring(1));
				AndroidFileClient.GetInstalledActivities(Device.Substring(1)).ForEach(item => { if (!Receivers.Contains(item)) Receivers.Add(item); });
				if (Receivers.Count > 0)
				{
					foreach (string Line in Receivers)
					{
						Logger.LogInformation("{0}", Line);
					}
					System.Environment.Exit(0);
				}
				Logger.LogInformation("No packages found with receiver.");
				System.Environment.Exit(0);
			}
			if (ArgIndex < args.Length && args[ArgIndex] == "stop-all")
			{
				ArgIndex++;
				bool bWait = false;

				if (ArgIndex < args.Length && args[ArgIndex].StartsWith("-w"))
				{
					bWait = true;
				}

				AndroidFileClient.StopAnyServers(Device.Substring(1), Port, bWait);
				System.Environment.Exit(0);
			}

			AndroidFileClient? client = CreateAndConnectClient(Device.Substring(1), PackageName, Token, IPAddress, Port);
			if (client == null)
			{
				System.Environment.Exit(1);
			}
			Logger.LogInformation("Connected!");

			if (ArgIndex < args.Length)
			{
				ArgIndex = ProcessCommand(client, false, args, ArgIndex);
				if (client != null)
				{
					client.CloseConnection();
				}
				System.Environment.Exit(0);
			}

			ShowHelp("");

			if (client != null)
			{
				client.CloseConnection();
			}
			System.Environment.Exit(0);
		}
	}
}
