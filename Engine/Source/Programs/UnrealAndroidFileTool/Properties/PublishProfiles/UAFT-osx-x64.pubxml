<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121. 
-->
<Project ToolsVersion="4.0" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <Configuration>Release</Configuration>
    <Platform>Any CPU</Platform>
    <PublishDir>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool\osx-x64</PublishDir>
    <PublishProtocol>FileSystem</PublishProtocol>
    <TargetFramework>net8.0</TargetFramework>
    <RuntimeIdentifier>osx-x64</RuntimeIdentifier>
    <SelfContained>true</SelfContained>
    <PublishSingleFile>true</PublishSingleFile>
    <PublishReadyToRun>true</PublishReadyToRun>
    <PublishTrimmed>true</PublishTrimmed>
  </PropertyGroup>
</Project>