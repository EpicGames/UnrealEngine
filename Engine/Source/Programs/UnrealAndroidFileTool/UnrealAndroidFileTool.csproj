<Project Sdk="Microsoft.NET.Sdk">
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Configuration Condition=" '$(Configuration)' == '' ">Development</Configuration>
    <OutputType>Exe</OutputType>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <Configurations>Debug;Release;Development</Configurations>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <AssemblyName>UnrealAndroidFileTool</AssemblyName>
    <RootNamespace>UnrealAndroidFileTool</RootNamespace>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages> <!-- remove non english resource languages -->
    <AppendRuntimeIdentifierToOutputPath>false</AppendRuntimeIdentifierToOutputPath>
    <IsWindows Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Windows)))' == 'true'">true</IsWindows> 
    <IsOSX Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::OSX)))' == 'true'">true</IsOSX> 
    <IsLinux Condition="'$([System.Runtime.InteropServices.RuntimeInformation]::IsOSPlatform($([System.Runtime.InteropServices.OSPlatform]::Linux)))' == 'true'">true</IsLinux> 
    <DebugType>pdbonly</DebugType> 
    <Nullable>enable</Nullable>
  </PropertyGroup>
  <Choose>
    <When Condition="'$(IsLinux)'=='true'">
      <PropertyGroup>
        <RuntimeIdentifier>linux-x64</RuntimeIdentifier>
      </PropertyGroup>
    </When>
    <When Condition="'$(IsOSX)'=='true'">
      <PropertyGroup>
        <!-- Set runtie identifier for osx so that it produces a app host executable (consistency with the other platforms) -->
        <RuntimeIdentifier>osx-x64</RuntimeIdentifier>
      </PropertyGroup>
    </When>
  </Choose>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <Optimize>false</Optimize>
    <OutputPath>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool</OutputPath>
    <DocumentationFile>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool\UnrealAndroidFileTool.xml</DocumentationFile>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <Optimize>false</Optimize>
    <DebugType>pdbonly</DebugType>
    <PlatformTarget>AnyCPU</PlatformTarget>
    <ErrorReport>prompt</ErrorReport>
    <AllowUnsafeBlocks>false</AllowUnsafeBlocks>
    <Prefer32Bit>false</Prefer32Bit>
	<OutputPath>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool</OutputPath>
    <DocumentationFile>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool\UnrealAndroidFileTool.xml</DocumentationFile>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <OutputPath>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool</OutputPath>
    <DocumentationFile>..\..\..\Binaries\DotNET\Android\UnrealAndroidFileTool\UnrealAndroidFileTool.xml</DocumentationFile>
    <DebugSymbols>true</DebugSymbols>
    <DebugType>full</DebugType>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>

  <!-- The following will glob all platform extensions' UnrealAndroidFileTool source
  files and include them in the build. They will also appear in VisualStudio.
  RecursiveDir does not work in VS Mac (and maybe Linux?), so only use it in Windows. 
  See: https://developercommunity.visualstudio.com/content/problem/527478/recursivedir-causes-infinitely-nested-subdirectori.html -->
  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"></ProjectReference>
    <ProjectReference Include="..\AutomationTool\Android\Android.Automation.csproj" PrivateAssets="All"></ProjectReference>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.CodeAnalysis.CSharp" Version="4.13.0" />
    <PackageReference Include="OpenTracing" Version="0.12.1" />
    <PackageReference Include="System.Management" Version="10.0.6" />
    <PackageReference Include="System.ServiceProcess.ServiceController" Version="10.0.6" />
  </ItemGroup>
</Project>
